{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "MsQuic Runtime Plugin",
	"Description" : "Runtime plugin for the MsQuic library.",
	"Category" : "Runtime",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Mac"
	],
    "SupportedPrograms": [
		"UnrealFrontend",
		"UnrealMultiUserServer",
		"UnrealMultiUserSlateServer",
		"CrashReportClientEditor",
		"CoopMultiUserServer"
	],
	"Modules" : [
		{
			"Name" : "MsQuicRuntime",
			"Type" : "Runtime",
			"LoadingPhase" : "PreDefault",
			"PlatformAllowList": [ "Win64", "Linux", "Mac" ],
			"ProgramAllowList": [
				"UnrealFrontend",
				"UnrealMultiUserServer",
				"UnrealMultiUserSlateServer",
				"CrashReportClientEditor",
				"CoopMultiUserServer"
			]
		}
	],
	"Plugins": []
}
