// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MsQuicRuntime : ModuleRules
	{
		public MsQuicRuntime(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
                    "MsQuic"
				}
			);
		}
	}
}
