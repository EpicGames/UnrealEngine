// Copyright Epic Games, Inc. All Rights Reserved.

#include "MsQuicRuntimeModule.h"
#include "MsQuicRuntimePrivate.h"

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Misc/CoreMisc.h"
#include "Misc/App.h"
#include "Misc/Paths.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogMsQuicRuntime);

#define LOCTEXT_NAMESPACE "FMsQuicRuntimeModule"


bool FMsQuicRuntimeModule::InitRuntime()
{
	if (MsQuicLibraryHandle)
	{
		UE_LOGF(LogMsQuicRuntime, Display,
			"[MsQuicRuntimeModule] MsQuic DLL already loaded.");

		return true;
	}

	if (!LoadMsQuicDll())
	{
		UE_LOGF(LogMsQuicRuntime, Error,
			"[MsQuicRuntimeModule] Could not load MsQuic DLL.");

		return false;
	}

	return true;
}


void FMsQuicRuntimeModule::ShutdownModule()
{
	FreeMsQuicDll();
}


bool FMsQuicRuntimeModule::LoadMsQuicDll()
{
	const FString MsQuicBinariesDir = FPaths::Combine(
		*FPaths::EngineDir(), *MSQUIC_BINARIES_PATH);

	FString MsQuicLib = "";

#if PLATFORM_WINDOWS

	MsQuicLib = FPaths::Combine(*MsQuicBinariesDir,
		TEXT("win64/msquic.dll"));

#elif PLATFORM_LINUX

	MsQuicLib = FPaths::Combine(*MsQuicBinariesDir,
		TEXT("linux/libmsquic.so.2"));

#elif PLATFORM_MAC

	MsQuicLib = FPaths::Combine(*MsQuicBinariesDir,
		TEXT("macos/libmsquic.dylib"));

#endif

	MsQuicLibraryHandle = (MsQuicLib.IsEmpty())
		? nullptr : FPlatformProcess::GetDllHandle(*MsQuicLib);

	return MsQuicLibraryHandle != nullptr;
}


/**
 * Free the MsQuic DLL/So if the LibraryHandle is valid.
 */
void FMsQuicRuntimeModule::FreeMsQuicDll()
{
	if (MsQuicLibraryHandle)
	{
		FPlatformProcess::FreeDllHandle(MsQuicLibraryHandle);
		MsQuicLibraryHandle = nullptr;
	}
}


#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FMsQuicRuntimeModule, MsQuicRuntime);
