// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"


/** Declares a log category for this module. */
DECLARE_LOG_CATEGORY_EXTERN(LogMsQuicRuntime, Display, All);
