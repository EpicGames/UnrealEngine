// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"

#define UE_API MSQUICRUNTIME_API


class FMsQuicRuntimeModule
	: public IModuleInterface
{

public:

	/**
	 * Initializes the runtime by loading the MsQuic DLL if it's not loaded already.
	 */
	static UE_API bool InitRuntime();

public:

	// IModuleInterface interface override

	UE_API virtual void ShutdownModule() override;

private:

	/**
	 * Load the appropriate MsQuic DLL/So for this platform.
	 */
	static bool LoadMsQuicDll();

	/**
	 * Frees the DLL handle.
	 */
	static void FreeMsQuicDll();

private:

	/** Holds the MsQuic library dll handle. */
	static inline void* MsQuicLibraryHandle = nullptr;

	/** Defines the MsQuic version to be used. */
	static inline const FString MSQUIC_VERSION = "v220";

	/** Defines the MsQuic binaries path. */
	static inline const FString MSQUIC_BINARIES_PATH
		= "Binaries/ThirdParty/MsQuic/" + MSQUIC_VERSION;

};

#undef UE_API
